{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "LiveLinkXR",
	"Description": "Live Link plugin for using XR tracked devices",
	"Category": "Virtual Production",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": true,
	"IsExperimentalVersion": false,
	"Installed": false,
	"EnabledByDefault": false,
	"SupportedTargetPlatforms": [
		"Win64"
	],
	"Plugins": [
		{
			"Name": "OpenXR",
			"Enabled": true
		}
	],
	"Modules": [
		{
			"Name": "LiveLinkXR",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "LiveLinkXROpenXRExt",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit",
			"PlatformAllowList": [
				"Win64"
			]
		}
	]
}
