// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class LiveLinkXR : ModuleRules
{
	public LiveLinkXR(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"LiveLinkInterface",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"Engine",
				"HeadMountedDisplay",
				"LiveLinkXROpenXRExt",
				"OpenXRHMD",
				"Slate",
				"SlateCore",
			}
		);
	}
}
