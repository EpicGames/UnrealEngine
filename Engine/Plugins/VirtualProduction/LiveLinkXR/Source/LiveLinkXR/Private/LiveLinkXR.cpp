// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkXR.h"

#define LOCTEXT_NAMESPACE "LiveLinkXRModule"

DEFINE_LOG_CATEGORY(LogLiveLinkXR);

void FLiveLinkXRModule::StartupModule()
{
}

void FLiveLinkXRModule::ShutdownModule()
{
}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FLiveLinkXRModule, LiveLinkXR)