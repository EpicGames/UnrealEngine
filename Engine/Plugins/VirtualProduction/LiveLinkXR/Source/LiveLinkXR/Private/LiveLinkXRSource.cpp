// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkXRSource.h"

#include "Async/Async.h"
#include "Engine/Engine.h"
#include "HAL/RunnableThread.h"
#include "ILiveLinkClient.h"
#include "IXRTrackingSystem.h"
#include "LiveLinkSubjectSettings.h"
#include "LiveLinkXR.h"
#include "LiveLinkXROpenXRExt.h"
#include "LiveLinkXROpenXRExtModule.h"
#include "LiveLinkXRConnectionSettings.h"
#include "LiveLinkXRSourceSettings.h"
#include "Misc/App.h"
#include "Roles/LiveLinkTransformRole.h"
#include "Roles/LiveLinkTransformTypes.h"


#define LOCTEXT_NAMESPACE "LiveLinkXRSource"


FLiveLinkXRSource::FLiveLinkXRSource(const FLiveLinkXRConnectionSettings& InConnectionSettings)
	: ConnectionSettings(InConnectionSettings)
{
	SourceStatus = LOCTEXT("SourceStatus_NoData", "No data");
	SourceType = LOCTEXT("SourceType_XR", "XR");
	SourceMachineName = LOCTEXT("XRSourceMachineName", "Local XR");

	if (!GEngine->XRSystem.IsValid())
	{
		UE_LOGF(LogLiveLinkXR, Error, "LiveLinkXRSource: Couldn't find a valid XR System");
		return;
	}

	if (GEngine->XRSystem->GetSystemName() != FName(TEXT("OpenXR")))
	{
		UE_LOG(LogLiveLinkXR, Error, TEXT("LiveLinkXRSource: Couldn't find a compatible XR System - LiveLinkXR is only compatible with OpenXR"));
		return;
	}

	if (!FLiveLinkXROpenXRExtModule::IsAvailable())
	{
		UE_LOGF(LogLiveLinkXR, Error, "LiveLinkXRSource: FLiveLinkXROpenXRExtModule not available");
		return;
	}

	TSharedPtr<FLiveLinkXROpenXRExtension> XrExt = FLiveLinkXROpenXRExtModule::Get().GetExtension();
	if (!ensure(XrExt) || !XrExt->IsSupported())
	{
		UE_LOGF(LogLiveLinkXR, Error, "LiveLinkXRSource: Extension plugin failed to initialize");
		return;
	}

	DeferredStartDelegateHandle = FCoreDelegates::OnEndFrame.AddRaw(this, &FLiveLinkXRSource::Start);
}

FLiveLinkXRSource::~FLiveLinkXRSource()
{
	// This could happen if the object is destroyed before FCoreDelegates::OnEndFrame calls FLiveLinkXRSource::Start
	if (DeferredStartDelegateHandle.IsValid())
	{
		FCoreDelegates::OnEndFrame.Remove(DeferredStartDelegateHandle);
	}

	if (Client)
	{
		Client->OnLiveLinkSubjectAdded().Remove(OnSubjectAddedDelegate);
	}

	Stop();

	if (Thread != nullptr)
	{
		Thread->WaitForCompletion();
		delete Thread;
		Thread = nullptr;
	}
}

void FLiveLinkXRSource::ReceiveClient(ILiveLinkClient* InClient, FGuid InSourceGuid)
{
	Client = InClient;
	SourceGuid = InSourceGuid;

	OnSubjectAddedDelegate = Client->OnLiveLinkSubjectAdded().AddRaw(this, &FLiveLinkXRSource::OnLiveLinkSubjectAdded);
}

void FLiveLinkXRSource::InitializeSettings(ULiveLinkSourceSettings* Settings)
{
	SavedSourceSettings = Cast<ULiveLinkXRSourceSettings>(Settings);
	if (!ensure(SavedSourceSettings))
	{
		return;
	}

	LocalUpdateRateInHz = SavedSourceSettings->LocalUpdateRateInHz;
}

void FLiveLinkXRSource::Update()
{
}

bool FLiveLinkXRSource::IsSourceStillValid() const
{
	// Source is valid if we have a valid thread
	const bool bIsSourceValid = !bStopping && (Thread != nullptr);
	return bIsSourceValid;
}

bool FLiveLinkXRSource::RequestSourceShutdown()
{
	Stop();

	return true;
}

TSubclassOf<ULiveLinkSourceSettings> FLiveLinkXRSource::GetSettingsClass() const
{
	return ULiveLinkXRSourceSettings::StaticClass();
}

// FRunnable interface
void FLiveLinkXRSource::Start()
{
	check(DeferredStartDelegateHandle.IsValid());

	FCoreDelegates::OnEndFrame.Remove(DeferredStartDelegateHandle);
	DeferredStartDelegateHandle.Reset();
	
	SourceStatus = LOCTEXT("SourceStatus_Receiving", "Receiving");

	static std::atomic<int32> ReceiverIndex = 0;
	ThreadName = "LiveLinkXR Receiver ";
	ThreadName.AppendInt(++ReceiverIndex);

	Thread = FRunnableThread::Create(this, *ThreadName, 128 * 1024, TPri_AboveNormal, FPlatformAffinity::GetPoolThreadMask());
}

void FLiveLinkXRSource::Stop()
{
	bStopping = true;
}

uint32 FLiveLinkXRSource::Run()
{
	TWeakPtr<FLiveLinkXROpenXRExtension> WeakExt;
	if (ensure(FLiveLinkXROpenXRExtModule::IsAvailable()))
	{
		WeakExt = FLiveLinkXROpenXRExtModule::Get().GetExtension();
		ensure(WeakExt.IsValid());
	}

	TMap<FName, FTransform> SubjectPoses;
	double LastFrameTimeSec = -DBL_MAX;
	while (!bStopping)
	{
		// Send new poses at the user specified update rate
		const double FrameIntervalSec = 1.0 / LocalUpdateRateInHz;
		const double TimeNowSec = FApp::GetCurrentTime();
		if (TimeNowSec >= (LastFrameTimeSec + FrameIntervalSec))
		{
			LastFrameTimeSec = TimeNowSec;

			// Send new poses at the user specified update rate
			SubjectPoses.Reset();

			if (TSharedPtr<FLiveLinkXROpenXRExtension> XrExt = WeakExt.Pin())
			{
				XrExt->GetSubjectPoses(SubjectPoses);
			}
			else
			{
				UE_LOGF(LogLiveLinkXR, Error, "Source thread couldn't pin FLiveLinkXROpenXRExtension; stopping");
				bStopping = true;
				return 1;
			}

			for (const TPair<FName, FTransform>& SubjectPose : SubjectPoses)
			{
				const FName& SubjectName = SubjectPose.Key;

				FLiveLinkFrameDataStruct FrameData(FLiveLinkTransformFrameData::StaticStruct());
				FLiveLinkTransformFrameData* TransformFrameData = FrameData.Cast<FLiveLinkTransformFrameData>();
				TransformFrameData->Transform = SubjectPose.Value;

				// These don't change frame to frame, so they really should be in static data. However, there is no MetaData in LiveLink static data :(
				TransformFrameData->MetaData.StringMetaData.Add(FName(TEXT("DeviceControlId")), SubjectName.ToString());

				Send(&FrameData, SubjectName);
			}
		}

		FPlatformProcess::Sleep(0.001f);
	}
	
	return 0;
}

void FLiveLinkXRSource::Send(FLiveLinkFrameDataStruct* FrameDataToSend, FName SubjectName)
{
	if (bStopping || (Client == nullptr))
	{
		return;
	}

	if (!EncounteredSubjects.Contains(SubjectName))
	{
		// If the LiveLink client already knows about this subject, then it must have been added via a preset
		// Only new subjects should be set to rebroadcast by default. Presets should respect the existing settings
		if (!Client->GetSubjects(true, true).Contains(FLiveLinkSubjectKey(SourceGuid, SubjectName)))
		{
			SubjectsToRebroadcast.Add(SubjectName);
		}

		FLiveLinkStaticDataStruct StaticData(FLiveLinkTransformStaticData::StaticStruct());
		Client->PushSubjectStaticData_AnyThread({SourceGuid, SubjectName}, ULiveLinkTransformRole::StaticClass(), MoveTemp(StaticData));
		EncounteredSubjects.Add(SubjectName);
	}

	Client->PushSubjectFrameData_AnyThread({ SourceGuid, SubjectName }, MoveTemp(*FrameDataToSend));
}

void FLiveLinkXRSource::OnLiveLinkSubjectAdded(FLiveLinkSubjectKey InSubjectKey)
{
	// Set rebroadcast to true for any new subjects
	if (SubjectsToRebroadcast.Contains(InSubjectKey.SubjectName))
	{
		ULiveLinkSubjectSettings* SubjectSettings = Cast<ULiveLinkSubjectSettings>(Client->GetSubjectSettings(InSubjectKey));
		if (SubjectSettings)
		{
			SubjectSettings->bRebroadcastSubject = true;
		}
	}
}

const FString FLiveLinkXRSource::GetDeviceTypeName(EXRTrackedDeviceType DeviceType)
{
	switch ((int32)DeviceType)
	{
		case (int32)EXRTrackedDeviceType::Invalid:				return TEXT("Invalid");
		case (int32)EXRTrackedDeviceType::HeadMountedDisplay:	return TEXT("HMD");
		case (int32)EXRTrackedDeviceType::Controller:			return TEXT("Controller");
		case (int32)EXRTrackedDeviceType::Other:				return TEXT("Tracker");
	}

	return FString::Printf(TEXT("Unknown (%i)"), (int32)DeviceType);
}


#undef LOCTEXT_NAMESPACE
