// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkXRSourceFactory.h"
#include "LiveLinkXRSource.h"
#include "SLiveLinkXRSourceFactory.h"


#define LOCTEXT_NAMESPACE "LiveLinkXRSourceFactory"

FText ULiveLinkXRSourceFactory::GetSourceDisplayName() const
{
	return LOCTEXT("SourceDisplayName", "LiveLinkXR Source");	
}

FText ULiveLinkXRSourceFactory::GetSourceTooltip() const
{
	return LOCTEXT("SourceTooltip", "Allows creation of multiple LiveLink sources using the XR tracking system");
}

TSharedPtr<SWidget> ULiveLinkXRSourceFactory::BuildCreationPanel(FOnLiveLinkSourceCreated InOnLiveLinkSourceCreated) const
{
	return SNew(SLiveLinkXRSourceFactory)
		.OnConnectionSettingsAccepted(FOnLiveLinkXRConnectionSettingsAccepted::CreateUObject(this, &ULiveLinkXRSourceFactory::CreateSourceFromSettings, InOnLiveLinkSourceCreated));
}

TSharedPtr<ILiveLinkSource> ULiveLinkXRSourceFactory::CreateSource(const FString& ConnectionString) const
{
	FLiveLinkXRConnectionSettings ConnectionSettings;
	if (!ConnectionString.IsEmpty())
	{
		FLiveLinkXRConnectionSettings::StaticStruct()->ImportText(*ConnectionString, &ConnectionSettings, nullptr, PPF_None, GLog, TEXT("ULiveLinkXRSourceFactory"));
	}
	return MakeShared<FLiveLinkXRSource>(ConnectionSettings);
}

void ULiveLinkXRSourceFactory::CreateSourceFromSettings(FLiveLinkXRConnectionSettings InConnectionSettings, FOnLiveLinkSourceCreated OnSourceCreated) const
{
	FString ConnectionString;
	FLiveLinkXRConnectionSettings::StaticStruct()->ExportText(ConnectionString, &InConnectionSettings, nullptr, nullptr, PPF_None, nullptr);

	TSharedPtr<FLiveLinkXRSource> SharedPtr = MakeShared<FLiveLinkXRSource>(InConnectionSettings);
	OnSourceCreated.ExecuteIfBound(SharedPtr, MoveTemp(ConnectionString));
}

#undef LOCTEXT_NAMESPACE
