// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkXRConnectionSettings.h"
#include "Widgets/SCompoundWidget.h"

class FStructOnScope;
class IStructureDetailsView;

#if WITH_EDITOR
#endif //WITH_EDITOR


struct FLiveLinkXRConnectionSettings;

DECLARE_DELEGATE_OneParam(FOnLiveLinkXRConnectionSettingsAccepted, FLiveLinkXRConnectionSettings);

class SLiveLinkXRSourceFactory : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SLiveLinkXRSourceFactory)
	{}
		SLATE_EVENT(FOnLiveLinkXRConnectionSettingsAccepted, OnConnectionSettingsAccepted)
	SLATE_END_ARGS()

	void Construct(const FArguments& Args);


private:
	FLiveLinkXRConnectionSettings ConnectionSettings;

#if WITH_EDITOR
	TSharedPtr<FStructOnScope> StructOnScope;
	TSharedPtr<IStructureDetailsView> StructureDetailsView;
#endif //WITH_EDITOR

	FReply OnSettingsAccepted();
	FOnLiveLinkXRConnectionSettingsAccepted OnConnectionSettingsAccepted;
};
