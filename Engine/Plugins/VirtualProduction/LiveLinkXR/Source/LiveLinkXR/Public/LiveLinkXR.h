// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"

DECLARE_LOG_CATEGORY_EXTERN(LogLiveLinkXR, Log, All);

class FLiveLinkXRModule : public IModuleInterface
{
public:

	static FLiveLinkXRModule& Get()
	{
		return FModuleManager::Get().LoadModuleChecked<FLiveLinkXRModule>(TEXT("LiveLinkXR"));
	}

	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};
