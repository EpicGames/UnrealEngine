// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "LiveLinkXRConnectionSettings.generated.h"

USTRUCT()
struct LIVELINKXR_API FLiveLinkXRConnectionSettings
{
	GENERATED_BODY()

public:
	/** Track all tracker pucks */
	UPROPERTY(EditAnywhere, Category = "Connection Settings")
	bool bTrackTrackers = true;

	/** Track all controllers */
	UPROPERTY(EditAnywhere, Category = "Connection Settings")
	bool bTrackControllers = false;

	/** Track all HMDs */
	UPROPERTY(EditAnywhere, Category = "Connection Settings")
	bool bTrackHMDs = false;

	/** Update rate (in Hz) at which to read the tracking data for each device */
	UPROPERTY(EditAnywhere, Category = "Connection Settings", meta = (ClampMin = 1, ClampMax = 1000))
	uint32 LocalUpdateRateInHz = 60;
};
