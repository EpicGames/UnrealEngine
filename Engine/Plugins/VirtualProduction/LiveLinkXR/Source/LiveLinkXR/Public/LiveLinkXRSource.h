// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ILiveLinkSource.h"
#include "HAL/Runnable.h"
#include "IXRTrackingSystem.h" // for EXRTrackedDeviceType; no longer used but pending deprecation
#include "LiveLinkXRConnectionSettings.h"

#include <atomic>


class ILiveLinkClient;
class ULiveLinkXRSourceSettings;


class LIVELINKXR_API FLiveLinkXRSource
	: public ILiveLinkSource
	, public FRunnable
	, public TSharedFromThis<FLiveLinkXRSource>
{
public:
	FLiveLinkXRSource(const FLiveLinkXRConnectionSettings& InConnectionSettings);

	virtual ~FLiveLinkXRSource();

	// Begin ILiveLinkSource Interface
	
	virtual void ReceiveClient(ILiveLinkClient* InClient, FGuid InSourceGuid) override;
	virtual void InitializeSettings(ULiveLinkSourceSettings* Settings) override;
	virtual void Update() override;

	virtual bool IsSourceStillValid() const override;

	virtual bool RequestSourceShutdown() override;

	virtual FText GetSourceType() const override { return SourceType; };
	virtual FText GetSourceMachineName() const override { return SourceMachineName; }
	virtual FText GetSourceStatus() const override { return SourceStatus; }

	virtual TSubclassOf<ULiveLinkSourceSettings> GetSettingsClass() const override;

	// End ILiveLinkSource Interface

	// Begin FRunnable Interface

	virtual bool Init() override { return true; }
	virtual uint32 Run() override;
	void Start();
	virtual void Stop() override;
	virtual void Exit() override { }

	// End FRunnable Interface

	void Send(FLiveLinkFrameDataStruct* FrameDataToSend, FName SubjectName);

	UE_DEPRECATED(5.1, "LiveLinkXR no longer uses EXRTrackedDeviceType.")
	const FString GetDeviceTypeName(EXRTrackedDeviceType DeviceType);

private:
	// Callback when the a livelink subject has been added
	void OnLiveLinkSubjectAdded(FLiveLinkSubjectKey InSubjectKey);

private:
	ILiveLinkClient* Client = nullptr;

	// Our identifier in LiveLink
	FGuid SourceGuid;

	FText SourceType;
	FText SourceMachineName;
	FText SourceStatus;
	
	// Threadsafe flag for terminating the main thread loop
	std::atomic<bool> bStopping = false;
	
	// Thread to update poses from
	FRunnableThread* Thread = nullptr;
	
	// Name of the update thread
	FString ThreadName;
	
	// List of subjects we've already encountered
	TSet<FName> EncounteredSubjects;

	// List of subjects to automatically set to rebroadcast
	TSet<FName> SubjectsToRebroadcast;

	// Deferred start delegate handle.
	FDelegateHandle DeferredStartDelegateHandle;

	// frame counter for data
	int32 FrameCounter = 0;

	// Update rate (in Hz) at which to read the tracking data for each device
	std::atomic<uint32> LocalUpdateRateInHz;

	// Critical section protection tracked devices
	FCriticalSection TrackedDeviceCriticalSection;

	// Delegate for when the LiveLink client has ticked
	FDelegateHandle OnSubjectAddedDelegate;

	const FLiveLinkXRConnectionSettings ConnectionSettings;
	ULiveLinkXRSourceSettings* SavedSourceSettings = nullptr;
};
