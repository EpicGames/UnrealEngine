// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "LiveLinkSourceSettings.h"
#include "LiveLinkXRSourceSettings.generated.h"


UCLASS()
class LIVELINKXR_API ULiveLinkXRSourceSettings : public ULiveLinkSourceSettings
{
	GENERATED_BODY()

public:
	/** Update rate (in Hz) at which to read the tracking data for each device */
	UPROPERTY(EditAnywhere, Category="LiveLinkXR", meta=(ClampMin=1, ClampMax=1000))
	uint32 LocalUpdateRateInHz = 60;
};
