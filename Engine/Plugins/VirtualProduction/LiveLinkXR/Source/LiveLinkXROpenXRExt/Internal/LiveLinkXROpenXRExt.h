// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IOpenXRExtensionPlugin.h"
#include "OpenXRCore.h"


#if PLATFORM_WINDOWS
namespace Windows
{
	union LARGE_INTEGER;
}
#endif


class LIVELINKXROPENXREXT_API FLiveLinkXROpenXRExtension
	: public IOpenXRExtensionPlugin
	, public TSharedFromThis<FLiveLinkXROpenXRExtension>
{
public:
	FLiveLinkXROpenXRExtension();
	virtual ~FLiveLinkXROpenXRExtension();

	bool IsSupported() const { return bIsSupported; }
	void GetSubjectPoses(TMap<FName, FTransform>& OutPoseMap);

	//~ Begin IOpenXRExtensionPlugin interface
	virtual FString GetDisplayName() override
	{
		return FString(TEXT("LiveLinkXR"));
	}

	virtual bool GetOptionalExtensions(TArray<const ANSICHAR*>& OutExtensions) override;
	virtual void PostCreateInstance(XrInstance InInstance) override;
	virtual void PostCreateSession(XrSession InSession) override;
	virtual void OnDestroySession(XrSession InSession) override;
	//~ End IOpenXRExtensionPlugin interface

private:
	XrSpace GetReferenceSpace() const;
	XrResult GetXrTimeNow(XrTime& OutXrTime) const;

	bool LocateSpace(XrSpace InSpace, XrSpace InRefSpace, XrTime InTime, FTransform& OutTransform);

#if PLATFORM_WINDOWS
	typedef XrResult(XRAPI_PTR* PFN_xrConvertWin32PerformanceCounterToTimeKHR)(XrInstance Instance, const Windows::LARGE_INTEGER* PerformanceCounter, XrTime* Time);
	PFN_xrConvertWin32PerformanceCounterToTimeKHR PfnXrConvertWin32PerformanceCounterToTimeKHR = nullptr;
#endif

private:
	bool bIsSupported = false;

	XrInstance Instance = XR_NULL_HANDLE;

	XrReferenceSpaceType TrackingSpaceType = XR_REFERENCE_SPACE_TYPE_LOCAL;
	XrSpace LocalSpace = XR_NULL_HANDLE;
	XrSpace StageSpace = XR_NULL_HANDLE;

	double WorldToMetersScale;
};
