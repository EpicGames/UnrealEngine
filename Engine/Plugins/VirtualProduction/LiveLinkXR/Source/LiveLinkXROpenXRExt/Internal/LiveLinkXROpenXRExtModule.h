// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/LogMacros.h"
#include "Modules/ModuleManager.h"


DECLARE_LOG_CATEGORY_EXTERN(LogLiveLinkXROpenXRExt, Log, All);


class FLiveLinkXROpenXRExtension;


class LIVELINKXROPENXREXT_API FLiveLinkXROpenXRExtModule : public IModuleInterface
{
public:
	/**
	* Singleton-like access to this module's interface.  This is just for convenience!
	* Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	*
	* @return Returns singleton instance, loading the module on demand if needed
	*/
	static inline FLiveLinkXROpenXRExtModule& Get()
	{
		return FModuleManager::LoadModuleChecked<FLiveLinkXROpenXRExtModule>("LiveLinkXROpenXRExt");
	}

	/**
	* Checks to see if this module is loaded and ready.  It is only valid to call Get() if IsAvailable() returns true.
	*
	* @return True if the module is loaded and ready to use
	*/
	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded("LiveLinkXROpenXRExt");
	}

	TSharedPtr<FLiveLinkXROpenXRExtension> GetExtension();

private:
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

private:
	TSharedPtr<FLiveLinkXROpenXRExtension> OpenXrExt;
};
