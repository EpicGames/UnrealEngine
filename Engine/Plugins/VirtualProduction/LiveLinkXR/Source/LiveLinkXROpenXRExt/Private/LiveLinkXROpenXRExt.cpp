// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkXROpenXRExt.h"
#include "LiveLinkXROpenXRExtModule.h"

#include "Engine/Engine.h"
#include "IOpenXRHMD.h"
#include "IOpenXRHMDModule.h"
#include "IXRTrackingSystem.h"
#include "OpenXRCore.h"
#include "OpenXRPlatformRHI.h" // Needed to get openxr_platform.h with the correct #defines set

#if PLATFORM_WINDOWS
	#include "Microsoft/MinimalWindowsApi.h"
#endif


#define LOCTEXT_NAMESPACE "LiveLinkXROpenXR"


XrPath GetPath(XrInstance Instance, const char* PathString)
{
	XrPath Path = XR_NULL_PATH;
	XrResult Result = xrStringToPath(Instance, PathString, &Path);
	check(XR_SUCCEEDED(Result));
	return Path;
}

XrPath GetPath(XrInstance Instance, const FString& PathString)
{
	return GetPath(Instance, StringCast<ANSICHAR>(*PathString).Get());
}



FLiveLinkXROpenXRExtension::FLiveLinkXROpenXRExtension()
{
	RegisterOpenXRExtensionModularFeature();
}


FLiveLinkXROpenXRExtension::~FLiveLinkXROpenXRExtension()
{
	UnregisterOpenXRExtensionModularFeature();
}


void FLiveLinkXROpenXRExtension::GetSubjectPoses(TMap<FName, FTransform>& OutPoseMap)
{
	XrTime PoseTime;
	if (!XR_ENSURE(GetXrTimeNow(PoseTime)))
	{
		return;
	}

	XrSpace RefSpace = GetReferenceSpace();
	if (RefSpace == XR_NULL_HANDLE)
	{
		return;
	}

	TArray<int32> Devices;

	if (GEngine && GEngine->XRSystem.IsValid())
	{
		IXRTrackingSystem* XRTrackingSystem = GEngine->XRSystem.Get();
		IOpenXRHMD* OpenXRHMD = XRTrackingSystem->GetIOpenXRHMD();

		if (XRTrackingSystem->EnumerateTrackedDevices(Devices))
		{
			for (int32 DeviceId : Devices)
			{
				FName Path;
				FTransform Location;
				if (LocateSpace(OpenXRHMD->GetTrackedDeviceSpace(DeviceId), RefSpace, PoseTime, Location))
				{
					OutPoseMap.Emplace(FOpenXRPath(OpenXRHMD->GetTrackedDevicePath(DeviceId)), Location);
				}
			}
		}
	}
}


bool FLiveLinkXROpenXRExtension::GetOptionalExtensions(TArray<const ANSICHAR*>& OutExtensions)
{
#if PLATFORM_WINDOWS
	OutExtensions.Add(XR_KHR_WIN32_CONVERT_PERFORMANCE_COUNTER_TIME_EXTENSION_NAME);
#endif

	return true;
}


void FLiveLinkXROpenXRExtension::PostCreateInstance(XrInstance InInstance)
{
	Instance = InInstance;

	bIsSupported = false;

	if (!IOpenXRHMDModule::IsAvailable())
	{
		UE_LOGF(LogLiveLinkXROpenXRExt, Error, "IOpenXRHMDModule not available");
		return;
	}

#if PLATFORM_WINDOWS
	if (!IOpenXRHMDModule::Get().IsExtensionEnabled("XR_KHR_win32_convert_performance_counter_time"))
	{
		UE_LOGF(LogLiveLinkXROpenXRExt, Error, "XR_KHR_win32_convert_performance_counter_time extension not available");
		return;
	}

	if (!XR_ENSURE(xrGetInstanceProcAddr(Instance, "xrConvertWin32PerformanceCounterToTimeKHR",
		reinterpret_cast<PFN_xrVoidFunction*>(&PfnXrConvertWin32PerformanceCounterToTimeKHR))))
	{
		UE_LOG(LogLiveLinkXROpenXRExt, Error, TEXT("Couldn't resolve xrConvertWin32PerformanceCounterToTimeKHR"));
		return;
	}

	bIsSupported = true;
#else
	UE_LOGF(LogLiveLinkXROpenXRExt, Error, "Support not implemented for this platform");
	return;
#endif
}


void FLiveLinkXROpenXRExtension::PostCreateSession(XrSession InSession)
{
	// Cache WorldToMetersScale.
	static FName SystemName(TEXT("OpenXR"));
	if (ensure(GEngine->XRSystem.IsValid()) && ensure(GEngine->XRSystem->GetSystemName() == SystemName))
	{
		WorldToMetersScale = GEngine->XRSystem.Get()->GetWorldToMetersScale();
	}

	// Create reference spaces. (Borrowed from FOpenXRHMD::OnStereoStartup)
	uint32_t ReferenceSpacesCount;
	XR_ENSURE(xrEnumerateReferenceSpaces(InSession, 0, &ReferenceSpacesCount, nullptr));

	TArray<XrReferenceSpaceType> Spaces;
	Spaces.SetNum(ReferenceSpacesCount);
	for (XrReferenceSpaceType& SpaceIter : Spaces)
	{
		// Initialize spaces array with valid enum values (avoid triggering validation error).
		SpaceIter = XR_REFERENCE_SPACE_TYPE_VIEW;
	}
	XR_ENSURE(xrEnumerateReferenceSpaces(InSession, (uint32_t)Spaces.Num(), &ReferenceSpacesCount, Spaces.GetData()));
	ensure(ReferenceSpacesCount == Spaces.Num());

	XrReferenceSpaceCreateInfo SpaceInfo{ XR_TYPE_REFERENCE_SPACE_CREATE_INFO, nullptr };
	SpaceInfo.poseInReferenceSpace = ToXrPose(FTransform::Identity);

	ensure(Spaces.Contains(XR_REFERENCE_SPACE_TYPE_LOCAL));
	SpaceInfo.referenceSpaceType = XR_REFERENCE_SPACE_TYPE_LOCAL;
	XR_ENSURE(xrCreateReferenceSpace(InSession, &SpaceInfo, &LocalSpace));

	if (Spaces.Contains(XR_REFERENCE_SPACE_TYPE_STAGE))
	{
		TrackingSpaceType = XR_REFERENCE_SPACE_TYPE_STAGE;
		SpaceInfo.referenceSpaceType = TrackingSpaceType;
		XR_ENSURE(xrCreateReferenceSpace(InSession, &SpaceInfo, &StageSpace));
	}
}


void FLiveLinkXROpenXRExtension::OnDestroySession(XrSession InSession)
{
	if (LocalSpace != XR_NULL_HANDLE)
	{
		XR_ENSURE(xrDestroySpace(LocalSpace));
		LocalSpace = XR_NULL_HANDLE;
	}

	if (StageSpace != XR_NULL_HANDLE)
	{
		XR_ENSURE(xrDestroySpace(StageSpace));
		StageSpace = XR_NULL_HANDLE;
	}
}


XrSpace FLiveLinkXROpenXRExtension::GetReferenceSpace() const
{
	switch (TrackingSpaceType)
	{
		case XR_REFERENCE_SPACE_TYPE_STAGE: return StageSpace;
		case XR_REFERENCE_SPACE_TYPE_LOCAL: return LocalSpace;
		default: checkNoEntry(); return XR_NULL_HANDLE;
	}
}


XrResult FLiveLinkXROpenXRExtension::GetXrTimeNow(XrTime& OutXrTime) const
{
#if PLATFORM_WINDOWS
	if (ensure(Instance) && ensure(PfnXrConvertWin32PerformanceCounterToTimeKHR))
	{
		Windows::LARGE_INTEGER CyclesNow;
		Windows::QueryPerformanceCounter(&CyclesNow);

		return PfnXrConvertWin32PerformanceCounterToTimeKHR(Instance, &CyclesNow, &OutXrTime);
	}
#endif

	return XR_ERROR_FEATURE_UNSUPPORTED;
}

bool FLiveLinkXROpenXRExtension::LocateSpace(XrSpace InSpace, XrSpace InRefSpace, XrTime InTime, FTransform& OutTransform)
{
	XrSpaceLocation XrLocation{ XR_TYPE_SPACE_LOCATION, nullptr };
	if (XR_SUCCEEDED(xrLocateSpace(InSpace, InRefSpace, InTime, &XrLocation)))
	{
		if ((XrLocation.locationFlags & XR_SPACE_LOCATION_POSITION_VALID_BIT) == 0 ||
			(XrLocation.locationFlags & XR_SPACE_LOCATION_POSITION_TRACKED_BIT) == 0)
		{
			return false;
		}

		if ((XrLocation.locationFlags & XR_SPACE_LOCATION_ORIENTATION_VALID_BIT) == 0 ||
			(XrLocation.locationFlags & XR_SPACE_LOCATION_ORIENTATION_TRACKED_BIT) == 0)
		{
			return false;
		}

		OutTransform = FTransform(
			ToFQuat(XrLocation.pose.orientation), ToFVector(XrLocation.pose.position, WorldToMetersScale)
		);
		return true;
	}
	return false;
}

#undef LOCTEXT_NAMESPACE
