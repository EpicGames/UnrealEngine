// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkXROpenXRExtModule.h"
#include "LiveLinkXROpenXRExt.h"


DEFINE_LOG_CATEGORY(LogLiveLinkXROpenXRExt);


IMPLEMENT_MODULE(FLiveLinkXROpenXRExtModule, LiveLinkXROpenXRExt);


void FLiveLinkXROpenXRExtModule::StartupModule()
{
	OpenXrExt = MakeShared<FLiveLinkXROpenXRExtension>();
}


void FLiveLinkXROpenXRExtModule::ShutdownModule()
{
	OpenXrExt.Reset();
}


TSharedPtr<FLiveLinkXROpenXRExtension> FLiveLinkXROpenXRExtModule::GetExtension()
{
	return OpenXrExt;
}
